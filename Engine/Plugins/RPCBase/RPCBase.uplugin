{
	"FileVersion": 3,
	"Version": 8,
	"VersionName": "1.0",
	"FriendlyName": "Remote Procedure Calls Base",
	"Description": "RPC base framework that client applications can use to implement RPC libraries.",
	"Category": "Runtime",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "RPCBase",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}
