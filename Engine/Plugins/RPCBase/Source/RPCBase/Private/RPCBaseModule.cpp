// Copyright Epic Games, Inc. All Rights Reserved.

#include "RPCBaseModule.h"
#include "Modules/ModuleManager.h"

void FRPCBaseModule::StartupModule()
{
}

void FRPCBaseModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FRPCBaseModule, RPCBase);