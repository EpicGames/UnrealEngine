// Copyright Epic Games, Inc. All Rights Reserved.

#include "RPCLibraryBase.h"

#include "Misc/CommandLine.h"

DEFINE_LOG_CATEGORY_STATIC(LogRPCLibraryBase, Log, All);

URPCLibraryBase::URPCLibraryBase()
{

}

URPCLibraryBase::~URPCLibraryBase()
{
#if WITH_RPC_REGISTRY
	if (Registrator != nullptr)
	{
		Registrator->DeregisterHttpCallbacks();
		Registrator->RemoveFromRoot();
	}
#endif
}

void URPCLibraryBase::Initialize()
{
#if WITH_RPC_REGISTRY
	UE_LOGF(LogRPCLibraryBase, Display, "RPC library initialization");

	check(UExternalRpcRegistry::GetInstance());

	Registrator = NewObject<UExternalRpcRegistrationComponent>();
	FParse::Value(FCommandLine::Get(), TEXT("externalrpclistenaddress="), Registrator->ListenerAddress);
	FParse::Value(FCommandLine::Get(), TEXT("rpcsenderid="), Registrator->SenderID);

	Registrator->AddToRoot();

	// Default RPC "RegistrationFinished" that sets RegistrationFinished to true to be used as a "ready" flag for issuing RPCs
	RegisterRPC(
		TEXT("RegistrationFinished"),
		FHttpPath(TEXT("/rpclibraryybase/registrationfinished")),
		EHttpServerRequestVerbs::VERB_POST,
		FHttpRequestHandler::CreateUObject(this, &URPCLibraryBase::HandleRegistrationFinishedRoute));
#endif // WITH_RPC_REGISTRY
}

void URPCLibraryBase::StartListening()
{
#if WITH_RPC_REGISTRY
	static bool bListenersStarted = false;
	if (!bListenersStarted)
	{
		UE_LOGF(LogRPCLibraryBase, Display, "FHttpServerModule: calling StartAllListeners");
		FHttpServerModule::Get().StartAllListeners();
		bListenersStarted = true;
	}
#endif
}

#if WITH_RPC_REGISTRY
void URPCLibraryBase::RegisterRPC(FName RouteName, const FHttpPath& HttpPath, const EHttpServerRequestVerbs& RequestVerbs, const FHttpRequestHandler& Handler, bool bOverrideIfBound, FString OptionalCategory, FString OptionalContentType, TArray<FExternalRpcArgumentDesc> OptionalInArguments)
{
	if (!Registrator->RegisteredRoutes.Contains(RouteName))
	{
		UE_LOGF(LogRPCLibraryBase, Display, "Registering new route %ls at path %ls", *RouteName.ToString(), *HttpPath.GetPath());

		FExternalRouteInfo NewRouteInfo;
		NewRouteInfo.RouteName = RouteName;
		NewRouteInfo.RoutePath = HttpPath;
		NewRouteInfo.RequestVerbs = RequestVerbs;
		NewRouteInfo.InputContentType = MoveTemp(OptionalContentType);
		NewRouteInfo.ExpectedArguments = MoveTemp(OptionalInArguments);
		NewRouteInfo.RpcCategory = MoveTemp(OptionalCategory);
		Registrator->RegisterHttpCallback(MoveTemp(NewRouteInfo), Registrator->CreateRouteHandle(Handler), bOverrideIfBound);
	}
}

bool URPCLibraryBase::HandleRegistrationFinishedRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	FString ResponseStr;
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&ResponseStr);
	JsonWriter->WriteObjectStart();
	JsonWriter->WriteValue(TEXT("RegistrationFinished"), RegistrationFinished);
	JsonWriter->WriteObjectEnd();
	JsonWriter->Close();

	auto Response = Registrator->CreateSimpleResponse(true, ResponseStr, false);
	OnComplete(MoveTemp(Response));

	return true;
}

TUniquePtr<FHttpServerResponse> URPCLibraryBase::CreateSimpleResponse(bool bInWasSuccessful, FString InValue, bool bInFatal)
{
	return Registrator->CreateSimpleResponse(bInWasSuccessful, InValue, bInFatal);
}

bool URPCLibraryBase::DeserializeRequest(TSharedPtr<FJsonObject>& RootObject, const TSharedRef<TJsonReader<>>& JsonReader, const TSharedRef<TJsonWriter<>>& JsonWriter)
{
	if (!FJsonSerializer::Deserialize(JsonReader, RootObject))
	{
		FString ErrorMessage = TEXT("Cannot deserialize RootObject from request body.");
		UE_LOGF(LogRPCLibraryBase, Warning, "%ls", *ErrorMessage);
		JsonWriter->WriteValue(TEXT("Error"), ErrorMessage);
		JsonWriter->WriteObjectEnd();
		JsonWriter->Close();
		return false;
	}
	return true;
}

bool URPCLibraryBase::GetStringField(TSharedPtr<FJsonObject>& RootObject, FString FieldName, FString& OutFieldValue, const TSharedRef<TJsonWriter<>>& JsonWriter)
{
	if (!RootObject->TryGetStringField(FieldName, OutFieldValue))
	{
		FString ErrorMessage = FString::Printf(TEXT("Cannot get field %s from request body."), *FieldName);
		JsonWriter->WriteValue(TEXT("Error"), ErrorMessage);
		JsonWriter->WriteObjectEnd();
		JsonWriter->Close();
		return false;
	}
	return true;
}
#endif // WITH_RPC_REGISTRY
