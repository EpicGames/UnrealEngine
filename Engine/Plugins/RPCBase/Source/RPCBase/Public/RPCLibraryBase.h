// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"

#if WITH_RPC_REGISTRY
#include "ExternalRpcRegistrationComponent.h"
#include "ExternalRpcRegistry.h"
#include "HttpRequestHandler.h"
#include "HttpServerModule.h"
#endif

#include "RPCLibraryBase.generated.h"

#if WITH_RPC_REGISTRY
enum class EHttpServerRequestVerbs : uint16;
struct FHttpPath;
struct FHttpServerResponse;
struct FExternalRouteInfo;
struct FExternalRpcArgumentDesc;
#endif

UCLASS(MinimalAPI)
class URPCLibraryBase : public UObject
{
	GENERATED_BODY()
private:
#if WITH_RPC_REGISTRY
	UExternalRpcRegistrationComponent* Registrator;
#endif
public:	
	URPCLibraryBase();
	~URPCLibraryBase();
protected:
	void Initialize();
	void StartListening();
	bool RegistrationFinished = false;
#if WITH_RPC_REGISTRY
	void RegisterRPC(FName RouteName, const FHttpPath& HttpPath, const EHttpServerRequestVerbs& RequestVerb, const FHttpRequestHandler& Handler, bool bOverrideIfBound = false, FString OptionalCategory = {}, FString OptionalContentType = {}, TArray<FExternalRpcArgumentDesc> OptionalInArguments = TArray<FExternalRpcArgumentDesc>());
	bool HandleRegistrationFinishedRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	TUniquePtr<FHttpServerResponse> CreateSimpleResponse(bool bInWasSuccessful, FString InValue = "", bool bInFatal = false);
	bool DeserializeRequest(TSharedPtr<FJsonObject>& RootObject, const TSharedRef<TJsonReader<>>& JsonReader, const TSharedRef<TJsonWriter<>>& JsonWriter);
	bool GetStringField(TSharedPtr<FJsonObject>&  RootObject, FString FiledName, FString& OutFieldValue, const TSharedRef<TJsonWriter<>>& JsonWriter);
#endif
};