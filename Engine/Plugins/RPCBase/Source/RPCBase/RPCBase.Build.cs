// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RPCBase : ModuleRules
{
	public RPCBase(ReadOnlyTargetRules target) : base(target)
	{
		PrivateDependencyModuleNames.AddRange(
		[
			"Core",
			"CoreUObject",
			"Json"
		]);

		if (Target.Configuration < UnrealTargetConfiguration.Shipping)
		{
			PublicDependencyModuleNames.Add("ExternalRpcRegistry");
			PublicDependencyModuleNames.Add("HTTPServer");
			PublicDefinitions.Add("WITH_RPC_REGISTRY=1");
		}
		else
		{
			PublicDefinitions.Add("WITH_RPC_REGISTRY=0");
		}
	}
}